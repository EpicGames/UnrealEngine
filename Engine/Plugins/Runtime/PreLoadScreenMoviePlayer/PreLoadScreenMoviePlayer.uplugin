{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Pre-Load Screen Movie Player",
	"Description" : "Handles a default implementation of using a Pre-Load screen to display an engine loading movie.",
	"Category" : "PreLoadScreenMoviePlayer",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules": [
		{
			"Name": "PreLoadScreenMoviePlayer",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PreEarlyLoadingScreen",
			"PlatformAllowList": 
			[
        "Android",
        "IOS",
        "Win64"
			]
		}
	]
}
