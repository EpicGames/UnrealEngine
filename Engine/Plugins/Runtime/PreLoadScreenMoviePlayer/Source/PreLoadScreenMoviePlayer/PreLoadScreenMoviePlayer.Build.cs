// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

// This module must be loaded "PreLoadingScreen" in the .uproject file, otherwise it will not hook in time!

public class PreLoadScreenMoviePlayer : ModuleRules
{
    public PreLoadScreenMoviePlayer(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string[] {
                "Core",
                "RHI",
                "RenderCore",
                "MoviePlayer",
                "Slate",
                "SlateCore",
                "InputCore",
                "Projects",
                "HTTP",
                "BuildPatchServices",
                "Json",
                "Engine",
                "RenderCore",
                "ApplicationCore",
                "Analytics",
                "AnalyticsET",
                "PreLoadScreen",
                "MoviePlayer",
                "CoreUObject",
            }
		);
    }
}
