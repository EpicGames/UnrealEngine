// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoadMoviePlayerModuleBase.h"

#include "MoviePlayer.h"
#include "PreLoadScreenManager.h"
#include "PreLoadMoviePlayerScreenBase.h"


void FPreLoadMoviePlayerScreenModuleBase::RegisterMovieStreamer(TSharedPtr<class IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer)
{
    if (MoviePreLoadScreen.IsValid())
    {
        //Pass the streamer to our screen and then init with this new information
        MoviePreLoadScreen->RegisterMovieStreamer(InMovieStreamer);
    }
}

void FPreLoadMoviePlayerScreenModuleBase::UnRegisterMovieStreamer(TSharedPtr<class IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer)
{
    CleanUpMovieStreamer();
}

void FPreLoadMoviePlayerScreenModuleBase::CleanUpMovieStreamer()
{
    if (MoviePreLoadScreen.IsValid())
    {
        MoviePreLoadScreen->CleanUp();
    }

    MoviePreLoadScreen.Reset();
}

void FPreLoadMoviePlayerScreenModuleBase::StartupModule()
{
    MoviePreLoadScreen = MakeShared<FPreLoadMoviePlayerScreenBase>();
    
    if (FPreLoadScreenManager::Get())
    {
        FPreLoadScreenManager::Get()->RegisterPreLoadScreen(MoviePreLoadScreen);
    }

    FCoreDelegates::RegisterMovieStreamerDelegate.AddRaw(this, &FPreLoadMoviePlayerScreenModuleBase::RegisterMovieStreamer);
    FCoreDelegates::UnRegisterMovieStreamerDelegate.AddRaw(this, &FPreLoadMoviePlayerScreenModuleBase::UnRegisterMovieStreamer);
}

void FPreLoadMoviePlayerScreenModuleBase::ShutdownModule()
{
    CleanUpMovieStreamer();
}

IMPLEMENT_MODULE(FPreLoadMoviePlayerScreenModuleBase, PreLoadScreenMoviePlayer);
