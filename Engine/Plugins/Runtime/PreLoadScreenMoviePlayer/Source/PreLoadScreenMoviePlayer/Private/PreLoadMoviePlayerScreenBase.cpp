// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreLoadMoviePlayerScreenBase.h"
#include "PreLoadScreenManager.h"
#include "PreLoadMoviePlayerModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/SlateDelegates.h"
#include "HAL/FileManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SDPIScaler.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SViewport.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GlobalShader.h"
#include "ShaderCompiler.h"

class SDefaultPreLoadScreenMovieBorder : public SBorder
{
public:

    SLATE_BEGIN_ARGS(SDefaultPreLoadScreenMovieBorder)
        : _OnKeyDown()
    {}

        SLATE_EVENT(FPointerEventHandler, OnMouseButtonDown)
        SLATE_EVENT(FOnKeyDown, OnKeyDown)
        SLATE_DEFAULT_SLOT(FArguments, Content)

     SLATE_END_ARGS()

        /**
        * Construct this widget
        *
        * @param	InArgs	The declaration data for this widget
        */
	void Construct(const FArguments& InArgs)
    {
        OnKeyDownHandler = InArgs._OnKeyDown;

        SBorder::Construct(SBorder::FArguments()
            .BorderImage(FCoreStyle::Get().GetBrush(TEXT("BlackBrush")))
            .OnMouseButtonDown(InArgs._OnMouseButtonDown)
            .Padding(0)[InArgs._Content.Widget]);

    }

    /**
    * Set the handler to be invoked when the user presses a key.
    *
    * @param InHandler   Method to execute when the user presses a key
    */
    void SetOnOnKeyDown(const FOnKeyDown& InHandler)
    {
        OnKeyDownHandler = InHandler;
    }

    /**
    * Overrides SWidget::OnKeyDown()
    * executes OnKeyDownHandler if it is bound
    */
    FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
    {
        if (OnKeyDownHandler.IsBound())
        {
            // If a handler is assigned, call it.
            return OnKeyDownHandler.Execute(MyGeometry, InKeyEvent);
        }
        return SBorder::OnKeyDown(MyGeometry, InKeyEvent);
    }

    /**
    * Overrides SWidget::SupportsKeyboardFocus()
    * Must support keyboard focus to accept OnKeyDown events
    */
    bool SupportsKeyboardFocus() const override
    {
        return true;
    }

protected:

    FOnKeyDown OnKeyDownHandler;
};

void FPreLoadMoviePlayerScreenBase::Init()
{
	if (bInitialized || !MovieStreamer.IsValid())
	{
		UE_LOGF(LogPreloadMoviePlayer, Log, "Init failed for PreLoadMoviePlayerScreenBase");
		return;
	}

	InitSettingsFromConfig(GGameIni);
	if (MovieAttributes.MoviePaths.IsEmpty())
	{
		UE_LOGF(LogPreloadMoviePlayer, Log, "Init failed for PreLoadMoviePlayerScreenBase, no MoviePaths defined.");
		return;
	}

	// Initialize shaders, because otherwise they might not be guaranteed to exist at this point
	if (!FPlatformProperties::RequiresCookedData())
	{
		TArray<int32> ShaderMapIds;
		ShaderMapIds.Add(GlobalShaderMapId);
		GShaderCompilingManager->FinishCompilation(TEXT("Global"), ShaderMapIds);
	}

	bInitialized = true;

	MoviePlayerContents = SNew(SDefaultPreLoadScreenMovieBorder)
		.OnKeyDown(this, &FPreLoadMoviePlayerScreenBase::OnLoadingScreenKeyDown)
		.OnMouseButtonDown(this, &FPreLoadMoviePlayerScreenBase::OnLoadingScreenMouseButtonDown)
		[
			SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
						.WidthOverride(this, &FPreLoadMoviePlayerScreenBase::GetMovieWidth)
						.HeightOverride(this, &FPreLoadMoviePlayerScreenBase::GetMovieHeight)
						[
							SAssignNew(MovieViewport, SViewport)
								.EnableGammaCorrection(false)
								.Visibility(this, &FPreLoadMoviePlayerScreenBase::GetViewportVisibility)
						]
				]
		];

	MovieStreamer->Init(MovieAttributes.MoviePaths, ((EMoviePlaybackType)((int)MovieAttributes.PlaybackType)));

	MovieViewport->SetViewportInterface(MovieStreamer->GetViewportInterface().ToSharedRef());
	MovieViewport->SetActive(true);

	// Register the movie viewport so that it can receive user input.
	if (!FPlatformProperties::SupportsWindowedMode())
	{
		FSlateApplication::Get().RegisterGameViewport(MovieViewport.ToSharedRef());
	}

	UE_LOGF(LogPreloadMoviePlayer, Log, "Init completed for PreLoadMoviePlayerScreenBase");
}

void FPreLoadMoviePlayerScreenBase::OnPlay(TWeakPtr<SWindow> TargetWindow)
{
    FPreLoadScreenBase::OnPlay(TargetWindow);
    //Make sure Init has been called before we play a movie.
    if (!bInitialized)
    {
        Init();
    }

    LastPlayTime = FPlatformTime::Seconds();	
}

void FPreLoadMoviePlayerScreenBase::OnStop()
{
    CleanUp();
}

void FPreLoadMoviePlayerScreenBase::CleanUp()
{
    if (MovieStreamer.IsValid())
    {
        MovieStreamer->ForceCompletion();
        MovieStreamer->Cleanup();
    }

    MovieStreamer.Reset();
	MoviePlayerContents.Reset();

	if (MovieViewport.IsValid())
	{
		MovieViewport.Reset();
	}

	MovieStreamingIsDone.Set(1);

	if (MovieAttributes.IsValid())
	{
		MovieAttributes = FPreLoadMovieAttributes();
	}
}

void FPreLoadMoviePlayerScreenBase::Tick(float DeltaTime)
{
	// Make sure if focus is lost for some reason it is restored to allow skipping
	if (MoviePlayerContents.IsValid() && !MoviePlayerContents->HasAnyUserFocus())
	{
		FSlateApplication::Get().SetAllUserFocus(MoviePlayerContents);
	}

    const bool bAutoCompleteWhenLoadingCompletes = MovieAttributes.bAutoCompleteWhenLoadingCompletes;
    const bool bWaitForManualStop = MovieAttributes.bWaitForManualStop;
    const bool bEnforceMinimumTime = MovieAttributes.MinimumLoadingScreenDisplayTime >= 0.0f;

    //Check if we should be done with displaying the movie
    if ( IsEngineExitRequested() ||
            ((bWaitForManualStop && bUserCalledFinish)
             || (!bEnforceMinimumTime && bAutoCompleteWhenLoadingCompletes && bIsEngineLoadingFinished)
             || (bEnforceMinimumTime && (FPlatformTime::Seconds() - LastPlayTime) >= MovieAttributes.MinimumLoadingScreenDisplayTime)
            )
       )
    {
        MovieStreamingIsDone.Set(1);
    }
}

void FPreLoadMoviePlayerScreenBase::RenderTick(FRHICommandListImmediate& RHICmdList, float DeltaTime)
{
    if (MovieStreamingIsPrepared())
    {
		MovieStreamer->TickPreEngine();
		MovieStreamer->TickPostEngine();

        const bool bMovieIsDone = MovieStreamer->Tick(RHICmdList, DeltaTime);
        if (bMovieIsDone)
        {
            MovieStreamingIsDone.Set(1);
        }
		MovieStreamer->TickPostRender();
    }
}

bool FPreLoadMoviePlayerScreenBase::MovieStreamingIsPrepared() const
{
    return MovieStreamer.IsValid() && MovieAttributes.IsValid();
}

FVector2D FPreLoadMoviePlayerScreenBase::GetMovieSize() const
{
    const FVector2D ScreenSize = OwningWindow.IsValid() ? OwningWindow.Pin()->GetClientSizeInScreen() : FVector2D::ZeroVector;
    if (MovieStreamingIsPrepared())
    {
        const float MovieAspectRatio = MovieStreamer->GetAspectRatio();
        const float ScreenAspectRatio = ScreenSize.X / ScreenSize.Y;
        if (MovieAspectRatio < ScreenAspectRatio)
        {
            return FVector2D(ScreenSize.Y * MovieAspectRatio, ScreenSize.Y);
        }
        else
        {
            return FVector2D(ScreenSize.X, ScreenSize.X / MovieAspectRatio);
        }
    }

    // No movie, so simply return the size of the window
    return ScreenSize;
}

FOptionalSize FPreLoadMoviePlayerScreenBase::GetMovieWidth() const
{
    return GetMovieSize().X;
}

FOptionalSize FPreLoadMoviePlayerScreenBase::GetMovieHeight() const
{
    return GetMovieSize().Y;
}

EVisibility FPreLoadMoviePlayerScreenBase::GetSlateBackgroundVisibility() const
{
    return MovieStreamingIsPrepared() && !IsMovieStreamingFinished() ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FPreLoadMoviePlayerScreenBase::GetViewportVisibility() const
{
    return MovieStreamingIsPrepared() && !IsMovieStreamingFinished() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FPreLoadMoviePlayerScreenBase::OnLoadingScreenMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent)
{
    return OnAnyDown();
}

FReply FPreLoadMoviePlayerScreenBase::OnLoadingScreenKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent)
{
    return OnAnyDown();
}

FReply FPreLoadMoviePlayerScreenBase::OnAnyDown()
{
    if (MovieAttributes.bMoviesAreSkippable)
    {
        MovieStreamingIsDone.Set(1);
        if (MovieStreamer.IsValid())
        {
            MovieStreamer->ForceCompletion();
        }
    }

    if (IsMovieStreamingFinished())
    {
        bUserCalledFinish = true;
    }

    return FReply::Handled();
}

bool FPreLoadMoviePlayerScreenBase::IsMovieStreamingFinished() const
{
    return MovieStreamingIsDone.GetValue() != 0;
}

void FPreLoadMoviePlayerScreenBase::InitSettingsFromConfig(const FString& ConfigFileName)
{
    // fill out the attributes
    FPreLoadMovieAttributes MovieScreen;

    bool bWaitForMoviesToComplete = false;
    // Note: this code is executed too early so we cannot access UMoviePlayerSettings because the configs for that object have not been loaded and coalesced .  Have to read directly from the configs instead
    GConfig->GetBool(TEXT("/Script/MoviePlayer.MoviePlayerSettings"), TEXT("bWaitForMoviesToComplete"), bWaitForMoviesToComplete, GGameIni);
    GConfig->GetBool(TEXT("/Script/MoviePlayer.MoviePlayerSettings"), TEXT("bMoviesAreSkippable"), MovieScreen.bMoviesAreSkippable, GGameIni);

    MovieScreen.bAutoCompleteWhenLoadingCompletes = !bWaitForMoviesToComplete;

    TArray<FString> StartupMovies;
    GConfig->GetArray(TEXT("/Script/MoviePlayer.MoviePlayerSettings"), TEXT("StartupMovies"), StartupMovies, GGameIni);

    if (StartupMovies.Num() == 0)
    {
        StartupMovies.Add(TEXT("Default_Startup"));
    }

    // double check that the movies exist
    // We dont know the extension so compare against any file in the directory with the same name for now
    // @todo New Movie Player: movies should have the extension on them when set via the project settings
    TArray<FString> ExistingMovieFiles;
    IFileManager::Get().FindFiles(ExistingMovieFiles, *(FPaths::ProjectContentDir() + TEXT("Movies")));

    bool bHasValidMovie = false;
    for (const FString& Movie : StartupMovies)
    {
        bool bFound = ExistingMovieFiles.ContainsByPredicate(
            [&Movie](const FString& ExistingMovie)
        {
            return ExistingMovie.Contains(Movie);
        });

        if (bFound)
        {
            bHasValidMovie = true;
            MovieScreen.MoviePaths.Add(Movie);
        }
    }

    if (bHasValidMovie)
    {
        // now setup the actual loading screen
        MovieAttributes = MovieScreen;
    }
}

void FPreLoadMoviePlayerScreenBase::RegisterMovieStreamer(TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> MovieStreamerIn)
{
    MovieStreamer = MovieStreamerIn;
}