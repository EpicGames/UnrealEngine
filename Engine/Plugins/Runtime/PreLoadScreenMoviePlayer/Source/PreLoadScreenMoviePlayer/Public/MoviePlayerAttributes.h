// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "MoviePlayerAttributes.generated.h"

UENUM()
enum EMovieScreenPlaybackType : int
{
    /** Normal playback mode.  Play each movie in the play list a single time */
    MT_MS_Normal UMETA(DisplayName = "Normal Playback"),
    /** Looped playback mode.  Play all movies in the play list in order then start over until manually canceled */
    MT_MS_Looped UMETA(DisplayName = "Looped Playback"),
    /** Alternate Looped mode.  Play all of the movies in the play list and loop just the last movie until loading is finished. */
    MT_MS_LoadingLoop UMETA(DisplayName = "Looped Last Playback"),
    MT_MS_MAX UMETA(Hidden)
};

/** Struct of all the attributes a loading screen will have. */
struct FPreLoadMovieAttributes
{
    FPreLoadMovieAttributes()
        : MinimumLoadingScreenDisplayTime(-1.0f)
        , bAutoCompleteWhenLoadingCompletes(true)
        , bMoviesAreSkippable(true)
        , bWaitForManualStop(false)
        , PlaybackType(EMovieScreenPlaybackType::MT_MS_Normal) {}

    /** The movie paths local to the game's Content/Movies/ directory we will play. */
    TArray<FString> MoviePaths;

    /** The minimum time that a loading screen should be opened for. */
    float MinimumLoadingScreenDisplayTime;

    /** If true, the loading screen will disappear as soon as all movies are played and loading is done. */
    bool bAutoCompleteWhenLoadingCompletes;

    /** If true, movies can be skipped by clicking the loading screen as long as loading is done. */
    bool bMoviesAreSkippable;

    /** If true, movie playback continues until Stop is called. */
    bool bWaitForManualStop;

    /** Should we just play back, loop, etc.  NOTE: if the playback type is MT_LoopLast, then bAutoCompleteWhenLoadingCompletes will be togged on when the last movie is hit*/
    TEnumAsByte<EMovieScreenPlaybackType> PlaybackType;

    /** True if any movie paths. */
    bool IsValid() const { return (MoviePaths.Num() > 0); }
};