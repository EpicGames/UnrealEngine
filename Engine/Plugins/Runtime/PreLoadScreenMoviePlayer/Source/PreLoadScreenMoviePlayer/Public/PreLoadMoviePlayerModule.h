// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "PreLoadScreen.h"

#include "MoviePlayer.h"

/** Module interface for handling any PreLoad Movie Player Screens. Mainly used to play movies before/during engine load. **/
class IPreLoadMoviePlayerScreenModule : public IModuleInterface
{
public:
    /**
    * Singleton-like access to this module's interface.  This is just for convenience!
    * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
    *
    * @return Returns singleton instance, loading the module on demand if needed
    */
    static inline IPreLoadMoviePlayerScreenModule& Get()
    {
        return FModuleManager::LoadModuleChecked< IPreLoadMoviePlayerScreenModule >(GetModuleName());
    }

    /**
    * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
    *
    * @return True if the module is loaded and ready to use
    */
    static inline bool IsAvailable()
    {
        return FModuleManager::Get().IsModuleLoaded(GetModuleName());
    }

    /** Simple Getter for the static name of this module
    *
    *	@return The Module Name as const FName
    */
    static inline const FName GetModuleName()
    {
        static const FName ModuleName = TEXT("PreLoadMoviePlayerScreen");
        return ModuleName;
    }
};