// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PreLoadMoviePlayerModule.h"
#include "PreLoadMoviePlayerScreenBase.h"

class FPreLoadMoviePlayerScreenModuleBase : public IPreLoadMoviePlayerScreenModule
{
public:

    //IPreLoadMoviePlayerScreenModule Interface
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;

    virtual bool IsGameModule() const override
    {
        return true;
    }

    virtual void RegisterMovieStreamer(TSharedPtr<class IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer);
    virtual void UnRegisterMovieStreamer(TSharedPtr<class IMovieStreamer, ESPMode::ThreadSafe> InMovieStreamer);
    virtual void CleanUpMovieStreamer();

private:
    TSharedPtr<FPreLoadMoviePlayerScreenBase> MoviePreLoadScreen;
};
