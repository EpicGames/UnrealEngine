// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePlayerAttributes.h"
#include "PreLoadScreenBase.h"
#include "MoviePlayer.h"
#include "Widgets/SWindow.h"
#include "Widgets/SViewport.h"

DEFINE_LOG_CATEGORY_STATIC(LogPreloadMoviePlayer, Log, All);

class SViewport;

class FPreLoadMoviePlayerScreenBase : public FPreLoadScreenBase
{
public:

    virtual void OnPlay(TWeakPtr<SWindow> TargetWindow) override;
    virtual void OnStop() override;
    virtual void Tick(float DeltaTime) override;
    virtual void RenderTick(FRHICommandListImmediate& RHICmdList, float DeltaTime) override;

    virtual void Init() override;

    virtual TSharedPtr<SWidget> GetWidget() override { return MoviePlayerContents; }
    virtual const TSharedPtr<const SWidget> GetWidget() const override { return MoviePlayerContents; }

    virtual void CleanUp() override;

    virtual void SetMovieAttributes(const FPreLoadMovieAttributes& MovieAttributesIn) { MovieAttributes = MovieAttributesIn; }

    //Default behavior is just to see if we have an active widget. Should really overload with our own behavior to see if we are done displaying
    virtual bool IsDone() const override { return IsMovieStreamingFinished(); }

    virtual bool MovieStreamingIsPrepared() const;
    virtual FReply OnLoadingScreenMouseButtonDown(const FGeometry& Geometry, const FPointerEvent& PointerEvent);
    virtual FReply OnLoadingScreenKeyDown(const FGeometry& Geometry, const FKeyEvent& KeyEvent);

    /** Callbacks for movie viewport */
    virtual FVector2D GetMovieSize() const;
    virtual FOptionalSize GetMovieWidth() const;
    virtual FOptionalSize GetMovieHeight() const;
    virtual EVisibility GetSlateBackgroundVisibility() const;
    virtual EVisibility GetViewportVisibility() const;

    virtual bool IsMovieStreamingFinished() const;

    FPreLoadMoviePlayerScreenBase() {};
    virtual ~FPreLoadMoviePlayerScreenBase() override {};
    
    virtual void InitSettingsFromConfig(const FString& ConfigFileName) override;

    virtual void RegisterMovieStreamer(TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> MovieStreamerIn);

protected:
    FReply OnAnyDown();

    /** The last time a movie was started */
    double LastPlayTime;

    /** Attributes of the loading screen we are currently displaying */
    FPreLoadMovieAttributes MovieAttributes;

    TSharedPtr<IMovieStreamer, ESPMode::ThreadSafe> MovieStreamer;
    TSharedPtr<SViewport> MovieViewport;
    TSharedPtr<SWidget> MoviePlayerContents;

    /** True if all movies have successfully streamed and completed */
    FThreadSafeCounter MovieStreamingIsDone;

    /** User has called finish (needed if LoadingScreenAttributes.bAutoCompleteWhenLoadingCompletes is off) */
    bool bUserCalledFinish;

    bool bInitialized;
};